// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
float {DataInterfaceName}_Weight;
Buffer<float> {DataInterfaceName}_PositionDeltaBuffer;
Buffer<uint> {DataInterfaceName}_VertexMapBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

float3 ReadPositionDelta_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = {DataInterfaceName}_VertexMapBuffer[VertexIndex] * 3;
	float3 Delta = float3({DataInterfaceName}_PositionDeltaBuffer[BufferIndex], {DataInterfaceName}_PositionDeltaBuffer[BufferIndex + 1], {DataInterfaceName}_PositionDeltaBuffer[BufferIndex + 2]);
	return Delta * {DataInterfaceName}_Weight;
}
