// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "VertexDeltaModel.h"
#include "VertexDeltaModelVizSettings.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "UObject/GCObjectScopeGuard.h"
#include "PreviewScene.h"
#include "Components/SkeletalMeshComponent.h"
#include "GeometryCache.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"

namespace UE::VertexDeltaModelTests
{
/*
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVertexDeltaModelMainTest, "MLDeformer.VertexDeltaModel.MainTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
	bool FVertexDeltaModelMainTest::RunTest(const FString& Parameters)
	{
		using namespace UE::MLDeformer;

		UMLDeformerAsset* DeformerAsset = LoadObject<UMLDeformerAsset>(nullptr, TEXT("MLDeformerAsset'/VertexDeltaModel/Tests/Biceps/biceps_mld.biceps_mld'"));
		UTEST_NOT_NULL(TEXT("MLDeformerAsset load"), DeformerAsset);
		FGCObjectScopeGuard Guard(DeformerAsset);

		UVertexDeltaModel* VertexDeltaModel = Cast<UVertexDeltaModel>(DeformerAsset->GetModel());
		UTEST_NOT_NULL(TEXT("VertexDeltaModel load check"), VertexDeltaModel);

		UNeuralNetwork* NeuralNet = VertexDeltaModel->GetNeuralNetwork();
		UTEST_NOT_NULL(TEXT("Neuralnet check"), NeuralNet);

		UMLDeformerInputInfo* InputInfo = VertexDeltaModel->GetInputInfo();
		UTEST_NOT_NULL(TEXT("InputInfo check"), InputInfo);
		const int32 NumFloatsPerBone = 6;
		const int32 NumFloatsPerCurve = 1;
		const int64 NumInputInfoInputs = InputInfo->CalcNumNeuralNetInputs(NumFloatsPerBone, NumFloatsPerCurve);
		const int64 NumNetworkInputs = NeuralNet->GetInputTensor().Num();
		UTEST_EQUAL(TEXT("InputInfo input count check"), NumInputInfoInputs, NumNetworkInputs);

		const int32 NumInputBones = InputInfo->GetNumBones();
		UTEST_EQUAL(TEXT("InputInfo bone count check"), NumInputBones, 1);
		UTEST_EQUAL(TEXT("InputInfo bone names check"), InputInfo->GetBoneNames().Num(), 1);
		UTEST_EQUAL(TEXT("InputInfo bone names content check"), InputInfo->GetBoneName(0), FName("lowerarm_l"));
		UTEST_EQUAL(TEXT("InputInfo vertex count check"), InputInfo->GetNumBaseMeshVertices(), InputInfo->GetNumTargetMeshVertices());

		const FVertexMapBuffer& VertexMapBuffer = VertexDeltaModel->GetVertexMapBuffer();
		UTEST_TRUE(TEXT("VertexMapBuffer check"), VertexMapBuffer.IsInitialized());

#if WITH_EDITORONLY_DATA
		USkeletalMesh* SkeletalMesh = VertexDeltaModel->GetSkeletalMesh();
		UTEST_NOT_NULL(TEXT("SkeletalMesh check"), SkeletalMesh);

		UTEST_TRUE(TEXT("Model vs Network compatible check"), VertexDeltaModel->GetInputInfo()->IsCompatible(SkeletalMesh));

		UGeometryCache* GeomCache = VertexDeltaModel->GetGeometryCache();
		UTEST_NOT_NULL(TEXT("GeomCache check"), GeomCache);

		const FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		UTEST_NOT_NULL(TEXT("ImportedModel check"), ImportedModel);

		const TArray<int32>& VertexMap = VertexDeltaModel->GetVertexMap();
		UTEST_EQUAL(TEXT("VertexMap size check"), VertexMap.Num(), ImportedModel->LODModels[0].NumVertices);

		const int32 NumBaseMeshVerts = UMLDeformerModel::ExtractNumImportedSkinnedVertices(SkeletalMesh);
		const int32 NumGeomCacheVerts = ExtractNumImportedGeomCacheVertices(GeomCache);
		UTEST_EQUAL(TEXT("VertexCount check"), NumBaseMeshVerts, NumGeomCacheVerts);
		UTEST_EQUAL(TEXT("Model SkelMesh VertexCount check"), VertexDeltaModel->GetNumBaseMeshVerts(), NumBaseMeshVerts);
		UTEST_EQUAL(TEXT("Model TargetMesh VertexCount check"), VertexDeltaModel->GetNumTargetMeshVerts(), NumGeomCacheVerts);

		const int64 NumNetworkOutputs = NeuralNet->GetOutputTensor().Num();
		const int64 ExpectedNetworkOutput = NumGeomCacheVerts * 3;	// 3 floats per vertex.
		UTEST_EQUAL(TEXT("NeuralNet output size check"), NumNetworkOutputs, ExpectedNetworkOutput);
		UTEST_EQUAL(TEXT("Hidden Layers check"), VertexDeltaModel->GetNumHiddenLayers(), 3);

		UVertexDeltaModelVizSettings* VizSettings = Cast<UVertexDeltaModelVizSettings>(VertexDeltaModel->GetVizSettings());
		UTEST_NOT_NULL(TEXT("VizSettings check"), VizSettings);

		// Create a preview scene and spawn an actor.
		TUniquePtr<FPreviewScene> PreviewScene = MakeUnique<FPreviewScene>(FPreviewScene::ConstructionValues());
		UWorld* World = PreviewScene->GetWorld();
		UTEST_NOT_NULL(TEXT("World check"), World);
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), FName(TEXT("ML Deformer Test Actor")));	
		AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);
		UTEST_NOT_NULL(TEXT("Actor check"), Actor);

		// Add a skeletal mesh component.
		USkeletalMeshComponent* SkelMeshComponent = NewObject<USkeletalMeshComponent>(Actor);
		SkelMeshComponent->SetSkeletalMesh(SkeletalMesh);
		Actor->SetRootComponent(SkelMeshComponent);
		SkelMeshComponent->RegisterComponent();
		
		// Add an ML Deformer component and activate it.
		UMLDeformerComponent* MLDeformerComponent = NewObject<UMLDeformerComponent>(Actor);
		MLDeformerComponent->SetDeformerAsset(DeformerAsset);
		MLDeformerComponent->RegisterComponent();
		MLDeformerComponent->Activate();
		UTEST_EQUAL(TEXT("MLDeformerComponent weight check"), MLDeformerComponent->GetWeight(), 1.0f);
		UTEST_TRUE(TEXT("MLDeformerComponent SkelMeshComponent check"), MLDeformerComponent->GetSkeletalMeshComponent() == SkelMeshComponent);

		// Create and init the model instance.
		UMLDeformerModelInstance* ModelInstance = MLDeformerComponent->GetModelInstance();
		UTEST_NOT_NULL(TEXT("CreateModelInstance check"), ModelInstance);
		UTEST_TRUE(TEXT("ModelInstance GetModel check"), ModelInstance->GetModel() == VertexDeltaModel);
		UTEST_TRUE(TEXT("ModelInstance compatible check"), ModelInstance->IsCompatible());
#endif // WITH_EDITORONLY_DATA

		return true;
	}*/
}	// namespace UE::VertexDeltaModelTests

#endif	// #if WITH_DEV_AUTOMATION_TESTS
