// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexDeltaGraphDataInterface.h"
#include "VertexDeltaModel.h"
#include "Components/SkeletalMeshComponent.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "MLDeformerModel.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "SkeletalMeshDeformerHelpers.h"

#include "VertexDeltaModelInstance.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderGraphResources.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VertexDeltaGraphDataInterface)

TArray<FOptimusCDIPinDefinition> UVertexDeltaGraphDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "PositionDelta", "ReadPositionDelta", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}

TSubclassOf<UActorComponent> UVertexDeltaGraphDataInterface::GetRequiredComponentClass() const
{
	return UMLDeformerComponent::StaticClass();
}

void UVertexDeltaGraphDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPositionDelta"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FVertexDeltaGraphDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(float, Weight)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, PositionDeltaBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>, VertexMapBuffer)
END_SHADER_PARAMETER_STRUCT()

FString UVertexDeltaGraphDataInterface::GetDisplayName() const
{
	return TEXT("MLD Vertex Delta Model");
}

void UVertexDeltaGraphDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const \
{
	InOutBuilder.AddNestedStruct<FVertexDeltaGraphDataInterfaceParameters>(UID);
}

TCHAR const* UVertexDeltaGraphDataInterface::TemplateFilePath = TEXT("/Plugin/VertexDeltaModel/Private/VertexDeltaModel.ush");

TCHAR const* UVertexDeltaGraphDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UVertexDeltaGraphDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UVertexDeltaGraphDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UVertexDeltaGraphDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UVertexDeltaGraphDataProvider* Provider = NewObject<UVertexDeltaGraphDataProvider>();
	Provider->DeformerComponent = Cast<UMLDeformerComponent>(InBinding);
	return Provider;
}

FComputeDataProviderRenderProxy* UVertexDeltaGraphDataProvider::GetRenderProxy()
{
	return new UE::VertexDeltaModel::FVertexDeltaGraphDataProviderProxy(DeformerComponent);
}

namespace UE::VertexDeltaModel
{
	FVertexDeltaGraphDataProviderProxy::FVertexDeltaGraphDataProviderProxy(UMLDeformerComponent* DeformerComponent)
	{
		using namespace UE::MLDeformer;

		const UMLDeformerAsset* DeformerAsset = DeformerComponent ? DeformerComponent->GetDeformerAsset() : nullptr;
		const UMLDeformerModel* Model = DeformerAsset ? DeformerAsset->GetModel() : nullptr;
		UMLDeformerModelInstance* ModelInstance = DeformerComponent ? DeformerComponent->GetModelInstance() : nullptr;
		const UVertexDeltaModel* VertexDeltaModel = Cast<UVertexDeltaModel>(Model);
		
		Weight = 0.0f;
		if (Model && VertexDeltaModel && ModelInstance && ModelInstance->GetSkeletalMeshComponent())
		{
			VertexDeltaModelInstance = Cast<UVertexDeltaModelInstance>(ModelInstance);
			if (VertexDeltaModelInstance)
			{
				USkinnedMeshComponent* SkinnedMeshComp = ModelInstance->GetSkeletalMeshComponent();
				if (SkinnedMeshComp && SkinnedMeshComp->IsRegistered())
				{
					SceneInterface = SkinnedMeshComp->GetScene();
					ComponentId = SkinnedMeshComp->GetPrimitiveSceneId();
				}
				NNEModelInstanceRDG = VertexDeltaModelInstance->GetNNEModelInstanceRDG();

				if (NNEModelInstanceRDG)
				{
					Weight = DeformerComponent->GetWeight();
					VertexMapBufferSRV = VertexDeltaModel->GetVertexMapBuffer().ShaderResourceViewRHI;
				}
			}
		}
	}

	bool FVertexDeltaGraphDataProviderProxy::IsValid(FValidationData const& InValidationData) const
	{
		if (InValidationData.ParameterStructSize != sizeof(FVertexDeltaGraphDataInterfaceParameters))
		{
			return false;
		}

		FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		if (!SkeletalMeshObject || NNEModelInstanceRDG == nullptr || VertexMapBufferSRV == nullptr)
		{
			return false;
		}

		return true;
	}
	

	void FVertexDeltaGraphDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
	{
		if (VertexDeltaModelInstance && VertexDeltaModelInstance->IsValidForDataProvider())
		{
			Buffer = GraphBuilder.RegisterExternalBuffer(VertexDeltaModelInstance->GetOutputRDGBuffer());
			BufferSRV = GraphBuilder.CreateSRV(Buffer, PF_R32_FLOAT);
		}
	}

	void FVertexDeltaGraphDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
	{
		FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		const FSkeletalMeshRenderData& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
		const FSkeletalMeshLODRenderData* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(0);
		const TStridedView<FVertexDeltaGraphDataInterfaceParameters> ParameterArray = MakeStridedParameterView<FVertexDeltaGraphDataInterfaceParameters>(InDispatchData);
		for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
		{
			FVertexDeltaGraphDataInterfaceParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.NumVertices = LodRenderData->GetNumVertices();
			Parameters.Weight = Weight;
			Parameters.PositionDeltaBuffer = BufferSRV;
			Parameters.VertexMapBuffer = VertexMapBufferSRV;
		}
	}
}	// namespace UE::VertexDeltaModel
