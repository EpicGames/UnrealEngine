// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexDeltaGraphDebugDataInterface.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerVizSettings.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "Components/SkeletalMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderGraphResources.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(VertexDeltaGraphDebugDataInterface)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

TArray<FOptimusCDIPinDefinition> UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "HeatMapMode", "ReadHeatMapMode" });
	Defs.Add({ "HeatMapMax", "ReadHeatMapMax" });
	Defs.Add({ "GroundTruthLerp", "ReadGroundTruthLerp" });
	Defs.Add({ "PositionGroundTruth", "ReadPositionGroundTruth", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}

TSubclassOf<UActorComponent> UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetRequiredComponentClass() const
{
	return UMLDeformerComponent::StaticClass();
}

void UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadHeatMapMode"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadHeatMapMax"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadGroundTruthLerp"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPositionGroundTruth"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FVertexDeltaGraphDebugDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(int32, HeatMapMode)
	SHADER_PARAMETER(float, HeatMapMax)
	SHADER_PARAMETER(float, GroundTruthLerp)
	SHADER_PARAMETER(uint32, GroundTruthBufferSize)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float>, PositionGroundTruthBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>, VertexMapBuffer)
END_SHADER_PARAMETER_STRUCT()

FString UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetDisplayName() const
{
	return TEXT("MLD Vertex Delta Model Debug");
}

void UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FVertexDeltaGraphDebugDataInterfaceParameters>(UID);
}

TCHAR const* UDEPRECATED_VertexDeltaGraphDebugDataInterface::TemplateFilePath = TEXT("/Plugin/VertexDeltaModel/Private/VertexDeltaModelHeatMap.ush");

TCHAR const* UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UDEPRECATED_VertexDeltaGraphDebugDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UDEPRECATED_VertexDeltaGraphDebugDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UDEPRECATED_VertexDeltaGraphDebugDataProvider* Provider = NewObject<UDEPRECATED_VertexDeltaGraphDebugDataProvider>();
	Provider->DeformerComponent = Cast<UMLDeformerComponent>(InBinding);
	if (Provider->DeformerComponent)
	{
		Provider->DeformerAsset = Provider->DeformerComponent->GetDeformerAsset();
	}

	#if WITH_EDITORONLY_DATA
		if (Provider->DeformerAsset != nullptr)
		{
			Provider->Init();
		}
	#endif
	return Provider;
}

FComputeDataProviderRenderProxy* UDEPRECATED_VertexDeltaGraphDebugDataProvider::GetRenderProxy()
{
#if WITH_EDITORONLY_DATA
	if (DeformerComponent && DeformerAsset && DeformerComponent->GetModelInstance() && DeformerComponent->GetModelInstance()->IsValidForDataProvider())
	{
		UE::VertexDeltaModel::FVertexDeltaGraphDebugDataProviderProxy* Proxy = new UE::VertexDeltaModel::FVertexDeltaGraphDebugDataProviderProxy(DeformerComponent, DeformerAsset, this);
		UMLDeformerModel* Model = DeformerAsset->GetModel();
		if (Model->GetVizSettings()->GetHeatMapMode() == EMLDeformerHeatMapMode::GroundTruth)
		{
			const float SampleTime = DeformerComponent->GetModelInstance()->GetSkeletalMeshComponent()->GetPosition();
			Model->SampleGroundTruthPositions(SampleTime, Proxy->GetGroundTruthPositions());
		}
		Proxy->HandleZeroGroundTruthPositions();
		return Proxy;
	}
#endif

	// Return default invalid proxy.
	return new FComputeDataProviderRenderProxy();
}

#if WITH_EDITORONLY_DATA
namespace UE::VertexDeltaModel
{
	FVertexDeltaGraphDebugDataProviderProxy::FVertexDeltaGraphDebugDataProviderProxy(UMLDeformerComponent* DeformerComponent, UMLDeformerAsset* DeformerAsset, UDEPRECATED_VertexDeltaGraphDebugDataProvider* InProvider)
		: FComputeDataProviderRenderProxy()
	{
		Provider = InProvider;

		if (DeformerComponent && DeformerAsset)
		{
			const UMLDeformerModel* Model = DeformerAsset->GetModel();	
			const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
			const UMLDeformerModelInstance* ModelInstance = DeformerComponent->GetModelInstance();

			USkinnedMeshComponent* SkinnedMeshComp = ModelInstance->GetSkeletalMeshComponent();
			if (SkinnedMeshComp && SkinnedMeshComp->IsRegistered())
			{
				SceneInterface = SkinnedMeshComp->GetScene();
				ComponentId = SkinnedMeshComp->GetPrimitiveSceneId();
			}
			VertexMapBufferSRV = Model->GetVertexMapBuffer().ShaderResourceViewRHI;
			HeatMapMode = static_cast<int32>(VizSettings->GetHeatMapMode());
			HeatMapMax = 1.0f / FMath::Max(VizSettings->GetHeatMapMax(), 0.00001f);
			GroundTruthLerp = VizSettings->GetGroundTruthLerp();
		}
	}

	void FVertexDeltaGraphDebugDataProviderProxy::HandleZeroGroundTruthPositions()
	{
		if (GroundTruthPositions.Num() == 0)
		{	
			// We didn't get valid ground truth vertices.
			// Make non empty array for later buffer generation.
			GroundTruthPositions.Add(FVector3f::ZeroVector);

			// Silently disable relevant debug things.
			if (HeatMapMode == static_cast<int32>(EMLDeformerHeatMapMode::GroundTruth))
			{
				HeatMapMode = -1;
				HeatMapMax = 0.0f;
				GroundTruthLerp = 0.0f;
			}
		}
	}

	bool FVertexDeltaGraphDebugDataProviderProxy::IsValid(FValidationData const& InValidationData) const
	{
		if (InValidationData.ParameterStructSize != sizeof(FVertexDeltaGraphDebugDataInterfaceParameters))
		{
			return false;
		}

		FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		if (!SkeletalMeshObject || VertexMapBufferSRV == nullptr)
		{
			return false;
		}

		return true;
	}

	void FVertexDeltaGraphDebugDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
	{
		GroundTruthBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(float), 3 * GroundTruthPositions.Num()), TEXT("MLDeformer.GroundTruthPositions"));
		GroundTruthBufferSRV = GraphBuilder.CreateSRV(GroundTruthBuffer);
		GraphBuilder.QueueBufferUpload(GroundTruthBuffer, GroundTruthPositions.GetData(), sizeof(FVector3f) * GroundTruthPositions.Num(), ERDGInitialDataFlags::None);
	}

	void FVertexDeltaGraphDebugDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
	{
		FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		const FSkeletalMeshRenderData& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
		const FSkeletalMeshLODRenderData* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(0);
		const TStridedView<FVertexDeltaGraphDebugDataInterfaceParameters> ParameterArray = MakeStridedParameterView<FVertexDeltaGraphDebugDataInterfaceParameters>(InDispatchData);
		for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
		{
			FVertexDeltaGraphDebugDataInterfaceParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.NumVertices = LodRenderData->GetNumVertices();
			Parameters.HeatMapMode = HeatMapMode;
			Parameters.HeatMapMax = HeatMapMax;
			Parameters.GroundTruthLerp = GroundTruthLerp;
			Parameters.GroundTruthBufferSize = GroundTruthPositions.Num();
			Parameters.PositionGroundTruthBuffer = GroundTruthBufferSRV;
			Parameters.VertexMapBuffer = VertexMapBufferSRV;
		}
	}
}	// namespace UE::VertexDeltaModel
#endif // WITH_EDITORONLY_DATA

PRAGMA_ENABLE_DEPRECATION_WARNINGS
