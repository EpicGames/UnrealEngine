// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexDeltaModel.h"
#include "VertexDeltaModelVizSettings.h"
#include "VertexDeltaModelInstance.h"
#include "MLDeformerComponent.h"
#include "MLDeformerAsset.h"
#include "MLDeformerObjectVersion.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeRDG.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"
#include "UObject/AssetRegistryTagsContext.h"

#define LOCTEXT_NAMESPACE "VertexDeltaModel"

VERTEXDELTAMODEL_API DEFINE_LOG_CATEGORY(LogVertexDeltaModel)

// Implement the module.
namespace UE::VertexDeltaModel
{
	class VERTEXDELTAMODEL_API FVertexDeltaModelModule
		: public IModuleInterface
	{
		void StartupModule() override
		{
			// Register an additional shader path for our shaders used inside the deformer graph system.
			const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("VertexDeltaModel"))->GetBaseDir(), TEXT("Shaders"));
			AddShaderSourceDirectoryMapping(TEXT("/Plugin/VertexDeltaModel"), PluginShaderDir);
		}
	};
}
IMPLEMENT_MODULE(UE::VertexDeltaModel::FVertexDeltaModelModule, VertexDeltaModel)

UVertexDeltaModel::UVertexDeltaModel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	SetVizSettings(ObjectInitializer.CreateEditorOnlyDefaultSubobject<UVertexDeltaModelVizSettings>(this, TEXT("VizSettings")));
#endif
}

void UVertexDeltaModel::SetNNEModelData(TObjectPtr<UNNEModelData> ModelData)
{
	NNEModel = ModelData;
	GetReinitModelInstanceDelegate().Broadcast();
}

UMLDeformerModelInstance* UVertexDeltaModel::CreateModelInstance(UMLDeformerComponent* Component)
{
	return NewObject<UVertexDeltaModelInstance>(Component);
}

FString UVertexDeltaModel::GetDefaultDeformerGraphAssetPath() const 
{ 
	return FString(TEXT("/VertexDeltaModel/Deformers/DG_VertexDeltaModel.DG_VertexDeltaModel"));
}

bool UVertexDeltaModel::IsTrained() const
{
	return NNEModel.Get() != nullptr;
}

void UVertexDeltaModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	#if WITH_EDITORONLY_DATA
		Context.AddTag(FAssetRegistryTag("MLDeformer.VertexDeltaModel.NumHiddenLayers", FString::FromInt(NumHiddenLayers), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.VertexDeltaModel.NumNeuronsPerLayer", FString::FromInt(NumNeuronsPerLayer), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.VertexDeltaModel.NumIterations", FString::FromInt(NumIterations), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.VertexDeltaModel.BatchSize", FString::FromInt(BatchSize), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.VertexDeltaModel.LearningRate", FString::Printf(TEXT("%f"), LearningRate), FAssetRegistryTag::TT_Numerical));
	#endif
}

#undef LOCTEXT_NAMESPACE
