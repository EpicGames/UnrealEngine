// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexDeltaModelInstance.h"
#include "VertexDeltaModel.h"
#include "MLDeformerAsset.h"
#include "RenderGraphBuilder.h"
#include "Components/SkeletalMeshComponent.h"

UVertexDeltaModel* UVertexDeltaModelInstance::GetVertexDeltaModel() const
{
	return Cast<UVertexDeltaModel>(Model);
}

UE::NNE::IModelInstanceRDG* UVertexDeltaModelInstance::GetNNEModelInstanceRDG() const
{
	return ModelInstanceRDG.Get();
}

TRefCountPtr<FRDGPooledBuffer> UVertexDeltaModelInstance::GetOutputRDGBuffer() const
{
	return RDGVertexDeltaBuffer; 
}

FString UVertexDeltaModelInstance::CheckCompatibility(USkeletalMeshComponent* InSkelMeshComponent, bool bLogIssues)
{
	FString ErrorString = Super::CheckCompatibility(InSkelMeshComponent, bLogIssues);

	// Verify the number of inputs versus the expected number of inputs.	
	const UE::NNE::IModelInstanceRDG* RDGModelInstance = GetNNEModelInstanceRDG();
	if (RDGModelInstance && Model->GetDeformerAsset())
	{
		TConstArrayView<UE::NNE::FTensorDesc> GPUTensorDesc = RDGModelInstance->GetInputTensorDescs();
		const int64 NumNeuralNetInputs = GPUTensorDesc[0].GetShape().GetData()[1];
		const int64 NumDeformerAssetInputs = static_cast<int64>(Model->GetInputInfo()->CalcNumNeuralNetInputs(Model->GetNumFloatsPerBone(), Model->GetNumFloatsPerCurve()));
		if (NumNeuralNetInputs != NumDeformerAssetInputs)
		{
			const FString InputErrorString = "The number of network inputs doesn't match the asset. Please retrain the asset."; 
			ErrorText += InputErrorString + "\n";
			if (bLogIssues)
			{
				UE_LOGF(LogVertexDeltaModel, Error, "Deformer '%ls': %ls", *(Model->GetDeformerAsset()->GetName()), *InputErrorString);
			}
		}
	}

	return ErrorString;
}

bool UVertexDeltaModelInstance::IsValidForDataProvider() const
{
	return ModelInstanceRDG.IsValid(); 
}

void UVertexDeltaModelInstance::Execute(float ModelWeight)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVertexDeltaModelInstance::Execute)

	if (ModelInstanceRDG)
	{
		ENQUEUE_RENDER_COMMAND(RunNeuralNetwork)
			(
				[this](FRHICommandListImmediate& RHICmdList)
				{
					ERHIPipeline Pipeline = RHICmdList.GetPipeline();

					if (Pipeline == ERHIPipeline::None)
					{
						RHICmdList.SwitchPipeline(ERHIPipeline::Graphics);
					}
					// Output deltas will be available on GPU for DeformerGraph via UMLDeformerDataProvider.
					FRDGBuilder GraphBuilder(RHICmdList);

					// Build Input Bindings
					TArray<UE::NNE::FTensorBindingRDG> InputBindingsRDG;
					UE::NNE::FTensorBindingRDG& BindingRDG = InputBindingsRDG.Emplace_GetRef();
					BindingRDG.Buffer = GraphBuilder.RegisterExternalBuffer(RDGInputBuffer);
					GraphBuilder.QueueBufferUpload(BindingRDG.Buffer, NNEInputTensorBuffer.GetData(), NNEInputTensorBuffer.Num() * sizeof(float), ERDGInitialDataFlags::NoCopy);
	
					// Build Output Bindings
					TArray<UE::NNE::FTensorBindingRDG> OutputBindingsRDG;
					UE::NNE::FTensorBindingRDG& OutputBindingRDG = OutputBindingsRDG.Emplace_GetRef();
					OutputBindingRDG.Buffer = GraphBuilder.RegisterExternalBuffer(RDGVertexDeltaBuffer);

					if (!ModelInstanceRDG.IsValid())
					{
						return;
					}
					ModelInstanceRDG->EnqueueRDG(GraphBuilder, InputBindingsRDG, OutputBindingsRDG);
	
					GraphBuilder.Execute();
				}
		);
	}
}

bool UVertexDeltaModelInstance::GetRDGVertexBufferDesc(TConstArrayView<UE::NNE::FTensorDesc>& InOutputTensorDescs, FRDGBufferDesc& OutBufferDesc)
{
	if (InOutputTensorDescs.Num() > 0)
	{
		const uint32 ElemByteSize = InOutputTensorDescs[0].GetElementByteSize();
		const UE::NNE::FSymbolicTensorShape& SymShape = InOutputTensorDescs[0].GetShape();
		for (int32 i = 1; i < InOutputTensorDescs.Num(); i++)
		{
			if (InOutputTensorDescs[i].GetElementByteSize() != ElemByteSize || SymShape != InOutputTensorDescs[i].GetShape())
			{
				return false;
			}
		}
		// Create a single flat output buffer
		const UE::NNE::FTensorShape OutputShape = UE::NNE::FTensorShape::MakeFromSymbolic(SymShape);
		OutBufferDesc.BytesPerElement = ElemByteSize;
		OutBufferDesc.NumElements = OutputShape.Volume() * InOutputTensorDescs.Num();
		OutBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::VertexBuffer;
		return true; 
	}
	return false;
}

void UVertexDeltaModelInstance::CreateRDGBuffers(TConstArrayView<UE::NNE::FTensorDesc>& OutputTensorDescs)
{
	ENQUEUE_RENDER_COMMAND(VertexDeltaModelInstance_CreateOuputRDGBuffer)
	(
		[this, &OutputTensorDescs](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder Builder(RHICmdList);
			FRDGBufferDesc VertexBufferDesc;
			if (GetRDGVertexBufferDesc(OutputTensorDescs, VertexBufferDesc))
			{
				FRDGBuffer* RDGBuffer = Builder.CreateBuffer(VertexBufferDesc, TEXT("UVertexDeltaModelInstance_OutputBuffer"));
				RDGVertexDeltaBuffer = Builder.ConvertToExternalBuffer(RDGBuffer);
			}

			FRDGBufferDesc InputDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), NNEInputTensorBuffer.Num());
			InputDesc.Usage = EBufferUsageFlags(InputDesc.Usage | BUF_SourceCopy);

			FRDGBuffer* RDGBuffer = Builder.CreateBuffer(InputDesc, TEXT("UVertexDeltaModelInstance_InputBuffer"), ERDGBufferFlags::None);
			RDGInputBuffer = Builder.ConvertToExternalBuffer(RDGBuffer);

			Builder.Execute();
		}
	);

	FRenderCommandFence RenderFence;
	RenderFence.BeginFence();
	RenderFence.Wait();
}

void UVertexDeltaModelInstance::PostMLDeformerComponentInit()
{
	if (!bNNECreationAttempted)
	{
		bNNECreationAttempted = true;
		CreateNNEModel();
	}
}

void UVertexDeltaModelInstance::CreateNNEModel()
{
	if (!ModelInstanceRDG.IsValid())
	{
		UVertexDeltaModel* VertexDeltaModel = Cast<UVertexDeltaModel>(Model);
		if (VertexDeltaModel)
		{
			const TWeakInterfacePtr<INNERuntime> Runtime = UE::NNE::GetRuntime<INNERuntime>(VertexDeltaModel->GetNNERuntimeName());
			const TWeakInterfacePtr<INNERuntimeRDG> RuntimeRDG = UE::NNE::GetRuntime<INNERuntimeRDG>(VertexDeltaModel->GetNNERuntimeName());
			if (!Runtime.IsValid())
			{
				UE_LOGF(LogVertexDeltaModel, Error, "Can't get NNE runtime: %ls", *VertexDeltaModel->GetNNERuntimeName());
				return;
			}

			// If we can create the model from its data.
			const TObjectPtr<UNNEModelData> ModelData = VertexDeltaModel->NNEModel;
			if (ModelData && RuntimeRDG.IsValid() && RuntimeRDG->CanCreateModelRDG(ModelData) == INNERuntimeRDG::ECanCreateModelRDGStatus::Ok)
			{
				// Create the model.
				const TSharedPtr<UE::NNE::IModelRDG> ModelRDG = RuntimeRDG->CreateModelRDG(ModelData);
				if (ModelRDG.IsValid())
				{
					ModelInstanceRDG = RuntimeRDG->CreateModelRDG(ModelData)->CreateModelInstanceRDG();
					if (ModelInstanceRDG)
					{
						// Setup inputs.
						const TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstanceRDG->GetInputTensorDescs();
						UE::NNE::FTensorShape InputTensorShape = UE::NNE::FTensorShape::MakeFromSymbolic(InputTensorDescs[0].GetShape());
						ModelInstanceRDG->SetInputTensorShapes({ InputTensorShape });
						check(InputTensorDescs[0].GetElementByteSize() == sizeof(float));
						NNEInputTensorBuffer.SetNumUninitialized(InputTensorShape.Volume());

						// Setup outputs.
						TConstArrayView<UE::NNE::FTensorDesc> OutputTensorDescs = ModelInstanceRDG->GetOutputTensorDescs();
						CreateRDGBuffers(OutputTensorDescs);
					}
					else
					{
						UE_LOGF(LogVertexDeltaModel, Error, "Failed to create NNE RDG Model for VertexDeltaModel.");
						return;
					}
				}
			}
		}
	}
}

bool UVertexDeltaModelInstance::SetupInputs()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVertexDeltaModelInstance::SetupInputs)

	// Some safety checks.
	if (!Model ||
		!SkeletalMeshComponent ||
		!SkeletalMeshComponent->GetSkeletalMeshAsset() ||
		!bIsCompatible)
	{
		return false;
	}

	// Get the network and make sure it's loaded.
	if (!ModelInstanceRDG)
	{
		return false;
	}

	const TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstanceRDG->GetInputTensorDescs();
	const UE::NNE::FTensorShape InputTensorShape = UE::NNE::FTensorShape::MakeFromSymbolic(InputTensorDescs[0].GetShape());
	const int64 NumNeuralNetInputs = InputTensorShape.Volume();
	const int64 NumDeformerAssetInputs = Model->GetInputInfo()->CalcNumNeuralNetInputs(Model->GetNumFloatsPerBone(), Model->GetNumFloatsPerCurve());
	if (NumNeuralNetInputs != NumDeformerAssetInputs)
	{
		return false;
	}

	if (NNEInputTensorBuffer.Num() != NumNeuralNetInputs)
	{
		return false; 
	}

	// Update and write the input values directly into the input tensor.
	const int64 NumFloatsWritten = SetNeuralNetworkInputValues(NNEInputTensorBuffer.GetData(), NNEInputTensorBuffer.Num());
	check(NumFloatsWritten == NumNeuralNetInputs);
	return true;
}

