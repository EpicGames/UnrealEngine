// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "SkeletalRenderPublic.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeRDG.h"
#include "VertexDeltaGraphDataInterface.generated.h"

class FRDGBuffer;
class FRDGBufferSRV;
class FRHIShaderResourceView;
class FSceneInterface;
class FSkeletalMeshObject;
class UMLDeformerComponent;
class USkeletalMeshComponent;
class UMLDeformerModel;
class UVertexDeltaModelInstance;

/** Compute Framework Data Interface for MLDefomer data. */
UCLASS(Category = ComputeFramework)
class VERTEXDELTAMODEL_API UVertexDeltaGraphDataInterface
	: public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	// UOptimusComputeDataInterface overrides.
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	// ~END UOptimusComputeDataInterface overrides.

	// UComputeDataInterface overrides.
	TCHAR const* GetClassName() const override { return TEXT("VertexDeltaModelData"); }
	virtual bool CanSupportUnifiedDispatch() const override { return true; }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	// ~END UComputeDataInterface overrides.

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for MLDeformer data. */
UCLASS(BlueprintType, EditInlineNew, Category = ComputeFramework)
class VERTEXDELTAMODEL_API UVertexDeltaGraphDataProvider
	: public UComputeDataProvider
{
	GENERATED_BODY()

public:
	/** The deformer component that this data provider works on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMLDeformerComponent> DeformerComponent = nullptr;

	// UComputeDataProvider overrides.
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	// ~END UComputeDataProvider overrides.
};

namespace UE::VertexDeltaModel
{
	/** Compute Framework Data Provider Proxy for MLDeformer data. */
	class VERTEXDELTAMODEL_API FVertexDeltaGraphDataProviderProxy
		: public FComputeDataProviderRenderProxy
	{
	public:
		FVertexDeltaGraphDataProviderProxy(UMLDeformerComponent* DeformerComponent);

		// FComputeDataProviderRenderProxy overrides.
		bool IsValid(FValidationData const& InValidationData) const override;
		void GatherDispatchData(FDispatchData const& InDispatchData) override;
		void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
		// ~END FComputeDataProviderRenderProxy overrides.

	protected:
		FSceneInterface* SceneInterface = nullptr;
		FPrimitiveComponentId ComponentId;
		UE::NNE::IModelInstanceRDG* NNEModelInstanceRDG = nullptr;
		UVertexDeltaModelInstance* VertexDeltaModelInstance = nullptr;
		FRHIShaderResourceView* VertexMapBufferSRV = nullptr;
		FRDGBuffer* Buffer = nullptr;
		FRDGBufferSRV* BufferSRV = nullptr;
		int32 NeuralNetworkInferenceHandle = -1;
		float Weight = 1.0f;
	};
}	// namespace UE::VertexDeltaModel
