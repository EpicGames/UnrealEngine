// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModel.h"
#include "PrimitiveComponentId.h"
#include "VertexDeltaGraphDataInterface.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "VertexDeltaGraphDebugDataInterface.generated.h"

class FRDGBuffer;
class FRDGBufferSRV;
class FRHIShaderResourceView;
class FSceneInterface;
class FSkeletalMeshObject;
class UMLDeformerAsset;
class USkeletalMeshComponent;

/** 
 * Compute Framework Data Interface for MLDeformer debugging data. 
 * This interfaces to editor only data, and so will only give valid results in that context.
 */
UCLASS(Deprecated, Category = ComputeFramework, meta = (DeprecationMessage = "Please use UMLDeformerGraphDebugDataInterface instead."))
class VERTEXDELTAMODEL_API UDEPRECATED_VertexDeltaGraphDebugDataInterface
	: public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	// UOptimusComputeDataInterface overrides.
	virtual FString GetDisplayName() const override;
	virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	// ~END UOptimusComputeDataInterface overrides.

	// UComputeDataInterface overrides.
	TCHAR const* GetClassName() const override { return TEXT("VertexDeltaModelDebugData"); }
	virtual bool CanSupportUnifiedDispatch() const override { return true; }
	virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& OutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	virtual TCHAR const* GetShaderVirtualPath() const override;
	virtual void GetShaderHash(FString& InOutKey) const override;
	virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	// ~END UComputeDataInterface overrides.

private:
	static TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for MLDeformer debugging data. */
UCLASS(Deprecated, BlueprintType, EditInlineNew, Category = ComputeFramework, meta = (DeprecationMessage = "Please use UMLDeformerGraphDebugDataProvider instead."))
class VERTEXDELTAMODEL_API UDEPRECATED_VertexDeltaGraphDebugDataProvider
	: public UComputeDataProvider
{
	GENERATED_BODY()

public:
	virtual void Init() {};

	// UComputeDataProvider overrides.
	virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	// ~END UComputeDataProvider overrides.

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMLDeformerComponent> DeformerComponent = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMLDeformerAsset> DeformerAsset = nullptr;
};

/** Compute Framework Data Provider Proxy for MLDeformer debugging data. */
#if WITH_EDITORONLY_DATA
namespace UE::VertexDeltaModel
{
	class VERTEXDELTAMODEL_API UE_DEPRECATED(5.2, "Please use UE::MLDeformer::FMLDeformerDebugDataProviderProxy instead.") FVertexDeltaGraphDebugDataProviderProxy
		: public FComputeDataProviderRenderProxy
	{
	public:
		FVertexDeltaGraphDebugDataProviderProxy(UMLDeformerComponent* DeformerComponent, UMLDeformerAsset* DeformerAsset, UDEPRECATED_VertexDeltaGraphDebugDataProvider* InProvider);

		virtual void HandleZeroGroundTruthPositions();

		// FComputeDataProviderRenderProxy overrides.
		bool IsValid(FValidationData const& InValidationData) const override;
		void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
		void GatherDispatchData(FDispatchData const& InDispatchData) override;
		// ~END FComputeDataProviderRenderProxy overrides.

		TArray<FVector3f>& GetGroundTruthPositions() { return GroundTruthPositions; }

	protected:
		TObjectPtr<UDEPRECATED_VertexDeltaGraphDebugDataProvider> Provider = nullptr;
		FSceneInterface* SceneInterface = nullptr;
		FPrimitiveComponentId ComponentId;
		TArray<FVector3f> GroundTruthPositions;
		FRHIShaderResourceView* VertexMapBufferSRV = nullptr;
		FRDGBuffer* GroundTruthBuffer = nullptr;
		FRDGBufferSRV* GroundTruthBufferSRV = nullptr;
		int32 HeatMapMode = 0;
		float HeatMapMax = 0.0f;
		float GroundTruthLerp = 0.0f;
	};
}	// namespace UE::VertexDeltaModel
#endif
