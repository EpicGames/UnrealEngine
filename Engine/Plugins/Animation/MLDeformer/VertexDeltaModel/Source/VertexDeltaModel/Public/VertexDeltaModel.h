// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheModel.h"
#include "VertexDeltaModelVizSettings.h"
#include "NNERuntimeRDG.h"
#include "VertexDeltaModel.generated.h"

VERTEXDELTAMODEL_API DECLARE_LOG_CATEGORY_EXTERN(LogVertexDeltaModel, Log, All);

class UNeuralNetwork;
class UNNEModelData;
/** 
 * The vertex delta model, which uses a GPU based neural network.
 * This model acts more as an example of how to implement a model that only uses the GPU.
 * It is not as efficient as the Neural Morph Model, even though that model runs on the CPU.
 * Unlike the Neural Morph Model this Vertex Delta Model does not generate any morph targets. All the vertex deltas are directly output
 * by the neural network. This buffer of output deltas remains on the GPU and is then used as input buffer directly inside an optimus deformer graph.
 */
UCLASS()
class VERTEXDELTAMODEL_API UVertexDeltaModel 
	: public UMLDeformerGeomCacheModel
{
	GENERATED_BODY()

public:
	UVertexDeltaModel(const FObjectInitializer& ObjectInitializer);

	// UObject overrides.
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	// UMLDeformerModel overrides.
	virtual FString GetDisplayName() const override			{ return "Vertex Delta Model"; }
	virtual UMLDeformerModelInstance* CreateModelInstance(UMLDeformerComponent* Component) override;
	virtual bool IsNeuralNetworkOnGPU() const override		{ return true; }	// GPU neural network.
	virtual FString GetDefaultDeformerGraphAssetPath() const override;
	virtual bool IsTrained() const override;
	// ~END UMLDeformerModel overrides.

#if WITH_EDITORONLY_DATA
	int32 GetNumHiddenLayers() const						{ return NumHiddenLayers; }
	int32 GetNumNeuronsPerLayer() const						{ return NumNeuronsPerLayer; }
	int32 GetNumIterations() const							{ return NumIterations; }
	int32 GetBatchSize() const								{ return BatchSize; }
	float GetLearningRate() const							{ return LearningRate; }
#endif

	void SetNNEModelData(TObjectPtr<UNNEModelData> ModelData);
	const FString GetNNERuntimeName() const					{ return TEXT("NNERuntimeORTDml"); }

public:
	/** The NNE neural network model. */
	UPROPERTY()
	TObjectPtr<UNNEModelData> NNEModel;

#if WITH_EDITORONLY_DATA
	/** The number of hidden layers that the neural network model will have.\nHigher numbers will slow down performance but can deal with more complex deformations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "1", ClampMax = "10"))
	int32 NumHiddenLayers = 3;

	/** The number of units/neurons per hidden layer. Higher numbers will slow down performance but allow for more complex mesh deformations. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "1"))
	int32 NumNeuronsPerLayer = 256;

	/** The number of iterations to train the model for. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "1"))
	int32 NumIterations = 10000;

	/** The number of frames per batch when training the model. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "1"))
	int32 BatchSize = 128;

	/** The learning rate used during the model training. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Settings", meta = (ClampMin = "0.000001", ClampMax = "1.0"))
	float LearningRate = 0.001f;
#endif // WITH_EDITORONLY_DATA
};
