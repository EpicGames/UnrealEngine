// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerModelInstance.h"
#include "NNE.h"
#include "NNERuntime.h"
#include "NNEModelData.h"
#include "NNERuntimeRDG.h"
#include "RenderGraphResources.h"
#include "VertexDeltaModelInstance.generated.h"

class UVertexDeltaModel;
class UNeuralNetwork;

UCLASS()
class VERTEXDELTAMODEL_API UVertexDeltaModelInstance
	: public UMLDeformerModelInstance
{
	GENERATED_BODY()

public:
	// UMLDeformerModelInstance overrides.
	virtual FString CheckCompatibility(USkeletalMeshComponent* InSkelMeshComponent, bool bLogIssues) override;
	virtual bool IsValidForDataProvider() const override;
	virtual void Execute(float ModelWeight) override;
	virtual bool SetupInputs() override;
	// ~END UMLDeformerModelInstance overrides.

	UVertexDeltaModel* GetVertexDeltaModel() const;
	virtual void PostMLDeformerComponentInit() override;

	/**
	 * Get the RDG Neural Network Model Instance
	 * @return The IModelInstanceRDG if successfully created, or nullptr
	 */
	UE::NNE::IModelInstanceRDG* GetNNEModelInstanceRDG() const;

	/**
	 * Get the output vertex delta buffer
	 * @return The FRDGBuffer for vertex deltas
	 */
	TRefCountPtr<FRDGPooledBuffer> GetOutputRDGBuffer() const;

	/**
	 * Get the render graph buffer description required for the output of a neural network. Return false if a flat buffer is not appropriate
	 * @return True if the OutputTensorDescs can be represented by a flat float  FRDGBuffer
	 */
	bool GetRDGVertexBufferDesc(TConstArrayView<UE::NNE::FTensorDesc>& InOutputTensorDescs, FRDGBufferDesc& OutBufferDesc);

private:

	void CreateRDGBuffers(TConstArrayView<UE::NNE::FTensorDesc>& OutputTensorDescs);
	void CreateNNEModel();

	// Input Buffer for Joint Matrices / Curve Floats
	TRefCountPtr<FRDGPooledBuffer> RDGInputBuffer;

	// Output Buffer for Vertex Deltas
	TRefCountPtr<FRDGPooledBuffer> RDGVertexDeltaBuffer;

	// The NNE RDG Model 
	TSharedPtr<UE::NNE::IModelInstanceRDG> ModelInstanceRDG;
	
	// The CPU Input Tensor Buffer
	TArray<float> NNEInputTensorBuffer;

	// Only attempt to create NNE Model once
	bool bNNECreationAttempted = false;
};
