// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheVizSettings.h"
#include "UObject/Object.h"
#include "VertexDeltaModelVizSettings.generated.h"

/**
 * The vizualization settings specific to the the vertex delta model.
 * This is inherited from the UMLDeformerGeomCacheVizSettings.
 * We cannot directly use that as we have to register the detail customization using a unique class for our model.
 * This is the reason why we just inherit a new class from it.
 */
UCLASS()
class VERTEXDELTAMODEL_API UVertexDeltaModelVizSettings 
	: public UMLDeformerGeomCacheVizSettings
{
	GENERATED_BODY()
};
