// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexDeltaEditorModel.h"
#include "VertexDeltaModel.h"
#include "VertexDeltaModelVizSettings.h"
#include "VertexDeltaTrainingModel.h"
#include "MLDeformerAsset.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

#define LOCTEXT_NAMESPACE "VertexDeltaEditorModel"

namespace UE::VertexDeltaModel
{
	using namespace UE::MLDeformer;

	FMLDeformerEditorModel* FVertexDeltaEditorModel::MakeInstance()
	{
		return new FVertexDeltaEditorModel();
	}

	UVertexDeltaModel* FVertexDeltaEditorModel::GetVertexDeltaModel() const
	{
		return Cast<UVertexDeltaModel>(Model);
	}

	ETrainingResult FVertexDeltaEditorModel::Train()
	{
		return TrainModel<UVertexDeltaTrainingModel>(this);
	}

	TObjectPtr<UNNEModelData> FVertexDeltaEditorModel::LoadNeuralNetworkFromOnnx(const FString& Filename) const
	{
		const FString OnnxFile = FPaths::ConvertRelativePathToFull(Filename);
		if (FPaths::FileExists(OnnxFile))
		{
			UE_LOGF(LogVertexDeltaModel, Display, "Loading Onnx file '%ls'...", *OnnxFile);
			TArray<uint8> RawModelData;
			const bool bIsModelInMem = FFileHelper::LoadFileToArray(RawModelData, *OnnxFile);
	
			if (bIsModelInMem)
			{
				TObjectPtr<UNNEModelData> ModelData = NewObject<UNNEModelData>(Model);
				ModelData->Init(FString("onnx"), RawModelData);
				UE_LOGF(LogVertexDeltaModel, Display, "Finished loading Onnx file '%ls'...", *OnnxFile);
				return ModelData;
			}
			else
			{
				UE_LOGF(LogVertexDeltaModel, Error, "Failed to load Onnx file '%ls'", *OnnxFile);
			}
		}
		else
		{
			UE_LOGF(LogVertexDeltaModel, Error, "Onnx file '%ls' does not exist!", *OnnxFile);
		}

		return TObjectPtr<UNNEModelData>();
	}

	bool FVertexDeltaEditorModel::LoadTrainedNetwork() const
	{
		const FString OnnxFile = GetTrainedNetworkOnnxFile();
		TObjectPtr<UNNEModelData> LoadedModelData = LoadNeuralNetworkFromOnnx(OnnxFile);
		if (LoadedModelData)
		{
			GetVertexDeltaModel()->SetNNEModelData(LoadedModelData);
			return true;
		}

		return false;
	}

	FString FVertexDeltaEditorModel::GetHeatMapDeformerGraphPath() const
	{
		return FString(TEXT("/VertexDeltaModel/Deformers/DG_VertexDeltaModel_HeatMap.DG_VertexDeltaModel_HeatMap"));
	}
}	// namespace UE::VertexDeltaModel

#undef LOCTEXT_NAMESPACE
