// Copyright Epic Games, Inc. All Rights Reserved.

#include "VertexDeltaModelDetails.h"
#include "VertexDeltaModel.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Layout/SBox.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "VertexDeltaModelDetails"

namespace UE::VertexDeltaModel
{
	using namespace UE::MLDeformer;

	TSharedRef<IDetailCustomization> FVertexDeltaModelDetails::MakeInstance()
	{
		return MakeShareable(new FVertexDeltaModelDetails());
	}

	void FVertexDeltaModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Create all the detail categories and add the properties of the base class.
		FMLDeformerGeomCacheModelDetails::CustomizeDetails(DetailBuilder);

		// Training settings.
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UVertexDeltaModel, NumHiddenLayers));
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UVertexDeltaModel, NumNeuronsPerLayer));
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UVertexDeltaModel, NumIterations));
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UVertexDeltaModel, BatchSize));
		TrainingSettingsCategoryBuilder->AddProperty(GET_MEMBER_NAME_CHECKED(UVertexDeltaModel, LearningRate));

		FDetailWidgetRow& ExampleWarningRow = TrainingSettingsCategoryBuilder->AddCustomRow(FText::FromString("VertexDeltaModelWarning"))
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(LOCTEXT("VertexDeltaModelWarningText", "The Vertex Delta Model is a GPU based example model that shouldn't be used in production. Please use the Neural Morph Model with the mode set to 'Global' to achieve similar results in production."))
				]
			];
	}
}	// namespace UE::VertexDeltaModel

#undef LOCTEXT_NAMESPACE
