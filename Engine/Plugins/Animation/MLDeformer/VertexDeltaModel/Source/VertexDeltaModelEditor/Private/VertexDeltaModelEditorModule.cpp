// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorModule.h"
#include "VertexDeltaModelVizSettingsDetails.h"
#include "VertexDeltaModelDetails.h"
#include "VertexDeltaEditorModel.h"
#include "VertexDeltaModel.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"

#define LOCTEXT_NAMESPACE "VertexDeltaModelEditorModule"

namespace UE::VertexDeltaModel
{
	class VERTEXDELTAMODELEDITOR_API FVertexDeltaModelEditorModule
		: public IModuleInterface
	{
	public:
		// IModuleInterface overrides.
		void StartupModule() override;
		void ShutdownModule() override;
		// ~END IModuleInterface overrides.
	};
}	// namespace UE::VertexDeltaModel

IMPLEMENT_MODULE(UE::VertexDeltaModel::FVertexDeltaModelEditorModule, VertexDeltaModelEditor)

namespace UE::VertexDeltaModel
{
	using namespace UE::MLDeformer;

	void FVertexDeltaModelEditorModule::StartupModule()
	{
		// Register object detail customizations.
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("VertexDeltaModelVizSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexDeltaModelVizSettingsDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("VertexDeltaModel", FOnGetDetailCustomizationInstance::CreateStatic(&FVertexDeltaModelDetails::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		// Register our custom ML deformer model to the model registry in the ML Deformer Framework.
		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
		ModelRegistry.RegisterEditorModel(UVertexDeltaModel::StaticClass(), FOnGetEditorModelInstance::CreateStatic(&FVertexDeltaEditorModel::MakeInstance), /*ModelPriority*/50);
	}

	void FVertexDeltaModelEditorModule::ShutdownModule()
	{
		// Unregister our ML Deformer model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("MLDeformerFrameworkEditor")))
		{
			FMLDeformerEditorModule& EditorModule = FModuleManager::GetModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();
			ModelRegistry.UnregisterEditorModel(UVertexDeltaModel::StaticClass());
		}

		// Unregister object detail customizations for this model.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("VertexDeltaModelVizSettings"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("VertexDeltaModel"));
			PropertyModule.NotifyCustomizationModuleChanged();
		}
	}

}	// namespace UE::VertexDeltaModel

#undef LOCTEXT_NAMESPACE
