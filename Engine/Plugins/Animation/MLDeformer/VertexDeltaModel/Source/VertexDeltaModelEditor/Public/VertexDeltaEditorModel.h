// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheEditorModel.h"

class UNeuralNetwork;
class UVertexDeltaModel;
class UNNEModelData;

namespace UE::VertexDeltaModel
{
	using namespace UE::MLDeformer;

	/** 
	 * The editor model for a UVertexDeltaModel.
	 */
	class VERTEXDELTAMODELEDITOR_API FVertexDeltaEditorModel 
		: public UE::MLDeformer::FMLDeformerGeomCacheEditorModel
	{
	public:
		// We need to implement this static MakeInstance method.
		static FMLDeformerEditorModel* MakeInstance();

		// FGCObject overrides.
		virtual FString GetReferencerName() const override	{ return TEXT("FVertexDeltaEditorModel"); }
		// ~END FGCObject overrides.
	
		// FMLDeformerEditorModel overrides.
		virtual ETrainingResult Train() override;
		virtual bool LoadTrainedNetwork() const override;
		virtual FString GetHeatMapDeformerGraphPath() const override;
		// ~END FMLDeformerEditorModel overrides.

		TObjectPtr<UNNEModelData> LoadNeuralNetworkFromOnnx(const FString& Filename) const;
		UVertexDeltaModel* GetVertexDeltaModel() const;
	};
}	// namespace UE::VertexDeltaModel
