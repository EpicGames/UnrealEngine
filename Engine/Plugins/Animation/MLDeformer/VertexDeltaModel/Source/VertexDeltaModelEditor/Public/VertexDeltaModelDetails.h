// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheModelDetails.h"

namespace UE::VertexDeltaModel
{
	/**
	 * The detail customization for the vertex delta model.
	 * This basically just adds the properties specific to the vertex delta model to the trainings settings.
	 * But as we inherit from the FMLDeformerGeomCacheModelDetails class, it directly creates all the other properties for us 
	 * as well, and inserts error messages when something is wrong, creates some groups, etc.
	 */
	class VERTEXDELTAMODELEDITOR_API FVertexDeltaModelDetails
		: public UE::MLDeformer::FMLDeformerGeomCacheModelDetails
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
		static TSharedRef<IDetailCustomization> MakeInstance();

		// ILayoutDetails overrides.
		void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		// ~END ILayoutDetails overrides.
	};
}	// namespace UE::VertexDeltaModel
