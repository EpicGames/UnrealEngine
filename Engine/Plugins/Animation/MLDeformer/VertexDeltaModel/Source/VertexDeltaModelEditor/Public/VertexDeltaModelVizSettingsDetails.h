// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheVizSettingsDetails.h"

namespace UE::VertexDeltaModel
{
	/**
	 * The detail customization for the vertex delta model visualization settings.
	 * We inherit from a base class as that already takes care of some nice grouping of properties, some error reporting
	 * in case of issues, and some nice grouping of settings.
	 */
	class VERTEXDELTAMODELEDITOR_API FVertexDeltaModelVizSettingsDetails
		: public UE::MLDeformer::FMLDeformerGeomCacheVizSettingsDetails
	{
	public:
		/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
		static TSharedRef<IDetailCustomization> MakeInstance()
		{
			return MakeShareable(new FVertexDeltaModelVizSettingsDetails());
		}
	};
}	// namespace UE::VertexDeltaModel
