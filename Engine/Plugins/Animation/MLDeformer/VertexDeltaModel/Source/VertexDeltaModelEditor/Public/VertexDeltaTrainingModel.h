// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheTrainingModel.h"
#include "VertexDeltaTrainingModel.generated.h"

/**
 * The training model for the vertex delta model.
 * This class is our link to the Python training.
 */
UCLASS(Blueprintable)
class VERTEXDELTAMODELEDITOR_API UVertexDeltaTrainingModel
	: public UMLDeformerGeomCacheTrainingModel
{
	GENERATED_BODY()

public:
	/**
	 * Main training function, with implementation in python.
	 * You need to implement this method. See the UMLDeformerTrainingModel documentation for more details.
	 * @see UMLDeformerTrainingModel
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "Python")
	int32 Train() const;
};
