// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class VertexDeltaModelEditor : ModuleRules
	{
		public VertexDeltaModelEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorFramework",
					"UnrealEd",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"EditorWidgets",
					"EditorStyle",
					"GeometryCache",
					"MLDeformerFramework",
					"MLDeformerFrameworkEditor",
					"VertexDeltaModel",
					"PropertyEditor",
					"NNE",
					"NNERuntimeORT",
					"ToolWidgets",
					"ComputeFramework"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
				}
			);
		}
	}
}
