{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.3",
	"FriendlyName": "ML Deformer Vertex Delta Model",
	"Description": "Vertex Delta Model for the ML Deformer Framework",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/5.0/en-US/using-the-machine-learning-deformer-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "VertexDeltaModel",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ]
		},
		{
			"Name": "VertexDeltaModelEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ]
		}
	],
	"Plugins": [
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "NNERuntimeORT",
			"Enabled": true
		},
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "MLDeformerFramework",
			"Enabled": true
		}
	],
	"PythonRequirements": [
		{
			"Platform": "All",
			"Requirements": [
				"onnx==1.14.1 --hash=sha256:776ab461515c20cc4e24dbd75af32b6b1e64de931dc5873b049f13bfec1c96e9 --hash=sha256:93e614edaf87ea1adba24663780ac62e30f421c117d695379daa9ff816de821b --hash=sha256:6b113fa0183034743e6477fec928e478a6d94eee8d9a4376c144d20d736cdc45 --hash=sha256:bdb15fc4b7f2a8a19abb52ac9672db876f9505e7219e206bcb7530e7c1274e55 --hash=sha256:4b9cd91b85cfbb0d6478f4a1a0aee4d95cf8839adc48c69130a0cf8452f21db4 --hash=sha256:17f78637d2f6c3c9afad0611fe4c583b6ba4839ac724af0846e5db24dc8dadc0 --hash=sha256:60ad73263a06056f9aa288b082887c6330be08475471c3a009f62439b2a67dca --hash=sha256:f131c2fd36f7848437be9de3b1fa5449a94245e16c6f275f66ac7cf8f183ec26 --hash=sha256:65672ae827ea5f0e59dc0d1cef1c0ed5083d5e8348946f98f1715ebb123573e9 --hash=sha256:921ad325b17484698d9d65978e123b1f351328ea50de6f84f25d09d5c7dde361 --hash=sha256:16a6667aff34431ab828b393ed8153c0a0cf15152d76f8d93aa48fb206217827 --hash=sha256:05d8609b4148f8ee4bd5d8186875ccb288300106242fc5201b8b575681bbd5c4"
			]
		}
	]
}